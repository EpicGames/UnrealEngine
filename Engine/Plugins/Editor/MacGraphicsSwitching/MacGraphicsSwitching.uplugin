{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Mac Graphics Switching",
	"Description" : "Provides support for switching between multiple graphics devices on macOS.",
	"Category" : "Misc",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MacGraphicsSwitching",
			"Type" : "Editor",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Mac"
			]
		}
	]
}
