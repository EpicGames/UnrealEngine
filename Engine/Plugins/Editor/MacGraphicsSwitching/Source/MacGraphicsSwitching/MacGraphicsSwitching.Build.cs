// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MacGraphicsSwitching : ModuleRules
	{
        public MacGraphicsSwitching(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
                    "InputCore",
					"LevelEditor",
					"Slate"
				}
			);

            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"PropertyEditor",
					"SlateCore",
					"EditorFramework",
					"UnrealEd"
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] 
				{
					"MainFrame",
					"Settings",
					"SettingsEditor"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] 
				{
					"MainFrame",
					"Settings",
					"SettingsEditor"
				}
			);
		}
	}
}
