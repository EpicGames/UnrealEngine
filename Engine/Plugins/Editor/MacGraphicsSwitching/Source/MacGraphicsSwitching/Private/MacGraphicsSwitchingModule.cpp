// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacGraphicsSwitchingModule.h"
#include "IMacGraphicsSwitchingModule.h"
#include "MacGraphicsSwitchingSettings.h"
#include "MacGraphicsSwitchingSettingsDetails.h"
#include "MacGraphicsSwitchingStyle.h"
#include "MacGraphicsSwitchingWidget.h"
#include "Features/IModularFeatures.h"
#include "Runtime/SlateCore/Public/Rendering/SlateRenderer.h"
#include "Editor/MainFrame/Public/Interfaces/IMainFrameModule.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "MacGraphicsSwitching"

class FMacGraphicsSwitchingModule : public IMacGraphicsSwitchingModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	void Initialize( TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog );
	void AddGraphicsSwitcher( FToolBarBuilder& ToolBarBuilder );
	
private:
	TSharedPtr< FExtender > NotificationBarExtender;
};

IMPLEMENT_MODULE(FMacGraphicsSwitchingModule, MacGraphicsSwitching)

void FMacGraphicsSwitchingModule::StartupModule()
{
	// Register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if( SettingsModule != nullptr )
	{
		SettingsModule->RegisterSettings( "Editor", "Plugins", "MacGraphicsSwitching",
										 LOCTEXT( "MacGraphicsSwitchingSettingsName", "Graphics Switching"),
										 LOCTEXT( "MacGraphicsSwitchingSettingsDescription", "Settings for macOS graphics switching"),
										 GetMutableDefault< UMacGraphicsSwitchingSettings >()
										 );
		
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("MacGraphicsSwitchingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FMacGraphicsSwitchingSettingsDetails::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();
		
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().AddRaw( this, &FMacGraphicsSwitchingModule::Initialize );
	}
}

void FMacGraphicsSwitchingModule::Initialize( TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog )
{
	if( !bIsRunningStartupDialog )
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().RemoveAll(this);

		FMacGraphicsSwitchingStyle::Initialize();
		
		bool MacUseAutomaticGraphicsSwitching = false;
		if ( GConfig->GetBool(TEXT("/Script/MacGraphicsSwitching.MacGraphicsSwitchingSettings"), TEXT("bShowGraphicsSwitching"), MacUseAutomaticGraphicsSwitching, GEditorSettingsIni) && MacUseAutomaticGraphicsSwitching )
		{
			// Switch to status bar
			/*NotificationBarExtender = MakeShareable( new FExtender() );
			NotificationBarExtender->AddToolBarExtension("Start",
														 EExtensionHook::After,
														 nullptr,
														 FToolBarExtensionDelegate::CreateRaw(this, &FMacGraphicsSwitchingModule::AddGraphicsSwitcher));
			
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
			LevelEditorModule.GetNotificationBarExtensibilityManager()->AddExtender( NotificationBarExtender );
			LevelEditorModule.BroadcastTitleBarMessagesChanged();*/
		}
	}
}

void FMacGraphicsSwitchingModule::AddGraphicsSwitcher( FToolBarBuilder& ToolBarBuilder )
{
	ToolBarBuilder.AddWidget( SNew( SMacGraphicsSwitchingWidget ).bLiveSwitching(true) );
}

void FMacGraphicsSwitchingModule::ShutdownModule()
{
	// Unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if( SettingsModule != nullptr )
	{
		SettingsModule->UnregisterSettings( "Editor", "Plugins", "MacGraphicsSwitching" );
	}
	
	if ( FModuleManager::Get().IsModuleLoaded("LevelEditor") && NotificationBarExtender.IsValid() )
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		// Switch to status bar
		//LevelEditorModule.GetNotificationBarExtensibilityManager()->RemoveExtender( NotificationBarExtender );
	}
	
	if ( FModuleManager::Get().IsModuleLoaded("MainFrame") )
	{
		FMacGraphicsSwitchingStyle::Shutdown();
		
		IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().RemoveAll(this);
	}
	
	if(FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomClassLayout("MacGraphicsSwitchingSettings");
	}
}

#undef LOCTEXT_NAMESPACE
