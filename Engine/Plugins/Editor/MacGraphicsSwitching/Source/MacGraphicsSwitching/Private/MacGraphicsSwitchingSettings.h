// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "MacGraphicsSwitchingSettings.generated.h"

UCLASS(Config=EditorSettings)
class UMacGraphicsSwitchingSettings : public UObject
{
	GENERATED_UCLASS_BODY()
	
public:
	
	UPROPERTY(Config, EditAnywhere, Category=RHI, meta = (ConfigRestartRequired=true))
	int32 RendererID;
	
	UPROPERTY(Config, EditAnywhere, Category=RHI, meta = (DisplayName = "Show Editor GPU Selector", ToolTip = "Adds a drop-down menu to the main Editor window allowing GPU selection. Requires restarting the Editor.", ConfigRestartRequired=true))
	bool bShowGraphicsSwitching;
};
