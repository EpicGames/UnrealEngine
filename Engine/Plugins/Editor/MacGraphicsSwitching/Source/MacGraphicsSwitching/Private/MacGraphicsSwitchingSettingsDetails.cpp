// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacGraphicsSwitchingSettingsDetails.h"
#include "MacGraphicsSwitchingModule.h"
#include "MacGraphicsSwitchingSettings.h"
#include "MacGraphicsSwitchingWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#define LOCTEXT_NAMESPACE "MacGraphicsSwitchingSettingsDetails"

TSharedRef<IDetailCustomization> FMacGraphicsSwitchingSettingsDetails::MakeInstance()
{
	return MakeShareable(new FMacGraphicsSwitchingSettingsDetails());
}

void FMacGraphicsSwitchingSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	TSharedRef<IPropertyHandle> PreferredRendererPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMacGraphicsSwitchingSettings, RendererID));
	DetailLayout.HideProperty("RendererID");

	IDetailCategoryBuilder& AccessorCategory = DetailLayout.EditCategory( "RHI" );
	AccessorCategory.AddCustomRow( LOCTEXT("PreferredRenderer", "Preferred Renderer") )
	.NameContent()
	[
		PreferredRendererPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(113)
	.MaxDesiredWidth(113)
	[
		SNew(SMacGraphicsSwitchingWidget)
		.bLiveSwitching(false)
		.PreferredRendererPropertyHandle(PreferredRendererPropertyHandle)
	];
}

#undef LOCTEXT_NAMESPACE
