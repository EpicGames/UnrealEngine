// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacGraphicsSwitchingStyle.h"
#include "MacGraphicsSwitchingModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FPaperStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

TSharedPtr< FSlateStyleSet > FMacGraphicsSwitchingStyle::StyleSet = NULL;
TSharedPtr< class ISlateStyle > FMacGraphicsSwitchingStyle::Get() { return StyleSet; }

void FMacGraphicsSwitchingStyle::Initialize()
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FLinearColor DisabledColor(FColor(0xaaaaaa));

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("MacGraphicsSwitchingStyle"));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		const FButtonStyle ComboBoxButton = FButtonStyle()
		.SetNormal(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(0.05f,0.05f,0.05f)))
		.SetHovered(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(0.05f,0.05f,0.05f)))
		.SetPressed(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(0.05f,0.05f,0.05f)))
		.SetDisabled(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), DisabledColor))
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));
		
		const FComboButtonStyle ComboButton = FComboButtonStyle()
		.SetButtonStyle(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
		.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
		.SetMenuBorderBrush(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(0.05f,0.05f,0.05f)))
		.SetMenuBorderPadding(FMargin(0.0f));
		
		const FComboButtonStyle ComboBoxComboButton = FComboButtonStyle(ComboButton)
		.SetButtonStyle(ComboBoxButton)
		.SetMenuBorderPadding(FMargin(1.0));
		
		StyleSet->Set("MacGraphicsSwitcher.ComboBox", FComboBoxStyle()
			.SetComboButtonStyle(ComboBoxComboButton)
			);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH
#undef BOX_BRUSH

void FMacGraphicsSwitchingStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
