// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FMacGraphicsSwitchingStyle
{
public:
	static void Initialize();

	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

private:
	static TSharedPtr< class FSlateStyleSet > StyleSet;
};
