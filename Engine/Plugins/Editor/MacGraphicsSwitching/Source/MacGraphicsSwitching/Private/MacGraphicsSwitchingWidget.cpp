// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacGraphicsSwitchingWidget.h"
#include "MacGraphicsSwitchingModule.h"
#include "MacGraphicsSwitchingStyle.h"
#include "Mac/MacGPUDescriptor.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SComboBox.h"
#include "Misc/ConfigCacheIni.h"
#include "ISettingsEditorModule.h"

#define LOCTEXT_NAMESPACE "MacGraphicsSwitchingWidget"

SMacGraphicsSwitchingWidget::SMacGraphicsSwitchingWidget()
: bLiveSwitching(false)
{
	// regenerate renderer list
	Renderers.Empty();
	
	FString DefaultDesc = FString(TEXT("System Default: ")) + FPlatformMisc::GetPrimaryGPUBrand();
	Renderers.Add(MakeShareable(new FRendererItem(FText::FromString(DefaultDesc), -1, 0)));
	
	TArray<FPlatformMisc::FGPUDescriptor> const& GPUs = FPlatformMisc::GetGPUDescriptors();
	for (FPlatformMisc::FGPUDescriptor const& GPU : GPUs)
	{
		FString RendererDesc = FString::Printf(TEXT("%d: %s"), GPU.GPUIndex, *FString(GPU.GPUName));
		
		Renderers.Add(MakeShareable(new FRendererItem(FText::FromString(RendererDesc), GPU.GPUIndex, GPU.RegistryID)));
	}
}

SMacGraphicsSwitchingWidget::~SMacGraphicsSwitchingWidget()
{
	
}

void SMacGraphicsSwitchingWidget::Construct(SMacGraphicsSwitchingWidget::FArguments const& InArgs)
{
	bLiveSwitching = InArgs._bLiveSwitching;
	
	if ( bLiveSwitching )
	{
		ChildSlot
		[
			SNew(SComboBox< TSharedPtr<FRendererItem>>)
			.ToolTipText(LOCTEXT("PreferredRendererToolTip", "Choose the preferred rendering device."))
			.ComboBoxStyle(FMacGraphicsSwitchingStyle::Get(), "MacGraphicsSwitcher.ComboBox")
			.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
			.OptionsSource(&Renderers)
			.OnSelectionChanged(this, &SMacGraphicsSwitchingWidget::OnSelectionChanged, InArgs._PreferredRendererPropertyHandle)
			.ContentPadding(2)
		 	.OnGenerateWidget(this, &SMacGraphicsSwitchingWidget::OnGenerateWidget)
			.OnComboBoxOpening(this, &SMacGraphicsSwitchingWidget::OnComboBoxOpening)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMacGraphicsSwitchingWidget::GetRendererText)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(SComboBox< TSharedPtr<FRendererItem>>)
			.ToolTipText(LOCTEXT("PreferredRendererToolTip", "Choose the preferred rendering device."))
			.OptionsSource(&Renderers)
			.OnSelectionChanged(this, &SMacGraphicsSwitchingWidget::OnSelectionChanged, InArgs._PreferredRendererPropertyHandle)
			.ContentPadding(2)
			.OnGenerateWidget(this, &SMacGraphicsSwitchingWidget::OnGenerateWidget)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMacGraphicsSwitchingWidget::GetRendererText)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	}
}

TSharedRef<SWidget> SMacGraphicsSwitchingWidget::OnGenerateWidget( TSharedPtr<FRendererItem> InItem )
{
	return SNew(STextBlock)
		.Text(InItem->Text);
}

void SMacGraphicsSwitchingWidget::OnComboBoxOpening()
{
	TArray<FPlatformMisc::FGPUDescriptor> const& GPUs = FPlatformMisc::GetGPUDescriptors();
	for (FPlatformMisc::FGPUDescriptor const& GPU : GPUs)
	{
		bool bFound = false;
		for (TSharedPtr<FRendererItem> const& Item : Renderers)
		{
			if (Item->RegistryID == GPU.RegistryID)
			{
				FString RendererDesc = FString::Printf(TEXT("%d: %s"), GPU.GPUIndex, *FString(GPU.GPUName));
				Item->Text = FText::FromString(RendererDesc);
				Item->RendererID = GPU.GPUIndex;
				bFound = true;
				break;
			}
		}
		
		if (!bFound)
		{
			FString RendererDesc = FString::Printf(TEXT("%d: %s"), GPU.GPUIndex, *FString(GPU.GPUName));
			Renderers.Add(MakeShareable(new FRendererItem(FText::FromString(RendererDesc), GPU.GPUIndex, GPU.RegistryID)));
		}
	}
	for (auto It = Renderers.CreateIterator(); It; ++It)
	{
		bool bFound = false;
		for (FPlatformMisc::FGPUDescriptor const& GPU : GPUs)
		{
			if ((*It)->RegistryID == GPU.RegistryID)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			It.RemoveCurrent();
		}
	}
}

void SMacGraphicsSwitchingWidget::OnSelectionChanged(TSharedPtr<FRendererItem> InItem, ESelectInfo::Type InSeletionInfo, TSharedPtr<IPropertyHandle> PreferredRendererPropertyHandle)
{
	if ( InItem.IsValid() )
	{
		if ( PreferredRendererPropertyHandle.IsValid() )
		{
			PreferredRendererPropertyHandle->SetValue(InItem->RendererID);
		}
		else if ( bLiveSwitching )
		{
			GConfig->SetInt(TEXT("/Script/MacGraphicsSwitching.MacGraphicsSwitchingSettings"), TEXT("RendererID"), InItem->RendererID, GEditorSettingsIni);
			
			ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");
			SettingsEditorModule.OnApplicationRestartRequired();
		}
		
		// Update all the contexts
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Mac.ExplicitRendererID"));
		if ( CVar )
		{
			CVar->Set( (int32)InItem->RendererID );
		}
	}
}

FText SMacGraphicsSwitchingWidget::GetRendererText() const
{
	FText Text = FText::FromString(FString::Printf(TEXT("System Default: %s"), *FPlatformMisc::GetPrimaryGPUBrand()));

	int32 ExplicitRendererId = 0;
	if ( bLiveSwitching )
	{
		ExplicitRendererId = FPlatformMisc::GetExplicitRendererIndex();
	}
	else
	{
		GConfig->GetInt(TEXT("/Script/MacGraphicsSwitching.MacGraphicsSwitchingSettings"), TEXT("RendererID"), ExplicitRendererId, GEngineIni);
	}
	
	if ( ExplicitRendererId )
	{
		for (auto Renderer: Renderers)
		{
			if ( Renderer->RendererID == ExplicitRendererId )
			{
				Text = Renderer->Text;
				break;
			}
		}
	}
	
	return Text;
}

#undef LOCTEXT_NAMESPACE
