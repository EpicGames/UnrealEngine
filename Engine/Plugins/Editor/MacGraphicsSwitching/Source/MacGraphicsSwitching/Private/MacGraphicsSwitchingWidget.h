// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMacGraphicsSwitchingModule.h"
#include "PropertyHandle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

struct FRendererItem
{
	FRendererItem(const FText& InText, int32 const InRendererID, uint64 const InRegistryID)
		: Text(InText)
		, RendererID(InRendererID)
		, RegistryID(InRegistryID)
	{
	}

	/** Text to display */
	FText Text;

	/** ID of the renderer */
	int32 RendererID;
	
	/** IOKit entry of the renderer */
	uint64 RegistryID;
};

class SMacGraphicsSwitchingWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMacGraphicsSwitchingWidget)
		: _bLiveSwitching(false) {}
	
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PreferredRendererPropertyHandle)
		SLATE_ARGUMENT(bool, bLiveSwitching)
	SLATE_END_ARGS();
	
	SMacGraphicsSwitchingWidget();
	virtual ~SMacGraphicsSwitchingWidget();
	
	void Construct(FArguments const& InArgs);
	
private:
	/** Generate a row widget for display in the list view */
	TSharedRef<SWidget> OnGenerateWidget( TSharedPtr<FRendererItem> InItem );
	
	/** Handle opening the combo box. */
	void OnComboBoxOpening();
	
	/** Set the renderer when we change selection */
	void OnSelectionChanged(TSharedPtr<FRendererItem> InItem, ESelectInfo::Type InSeletionInfo, TSharedPtr<IPropertyHandle> PreferredProviderPropertyHandle);
	
	/** Get the text to display on the renderer drop-down */
	FText GetRendererText() const;
	
private:
	/** Accessor names to display in a drop-down list */
	TArray<TSharedPtr<FRendererItem>> Renderers;
	/** Whether we are modifying the current or default */
	bool bLiveSwitching;
};
